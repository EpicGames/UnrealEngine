{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange OpenUSD Chaos Cloth Asset",
	"Description": "Allows translation of OpenUSD files with Chaos Cloth Asset schemas via Interchange",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"IsExperimentalVersion": true,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "InterchangeOpenUSDChaosClothAssetImport",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Interchange",
			"Enabled": true
		},
		{
			"Name": "InterchangeEditor",
			"Enabled": true,
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		},
		{
			"Name": "InterchangeChaosClothAsset",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "InterchangeOpenUSD",
			"Enabled": true
		},
		{
			"Name": "USDCore",
			"Enabled": true
		}
	]
}
