// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeOpenUSDChaosClothAssetImport : ModuleRules
	{
		public InterchangeOpenUSDChaosClothAssetImport(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "IUSDCA";

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Chaos",
					"ChaosClothAsset",
					"ChaosClothAssetDataflowNodes",
					"ChaosClothAssetEngine",
					"InterchangeChaosClothAssetImport",
					"InterchangeCore",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeNodes",
					"InterchangeOpenUSDImport",
					"InterchangePipelines",
					"MeshDescription",
					"StaticMeshDescription",
					"UnrealUSDWrapper",
					"USDClasses",
					"USDUtilities",
				}
			);

			// Explicitly add this internal folder as an include as moving the files out to "Public" would
			// require some deprecation procedures
			PrivateIncludePaths.Add(System.IO.Path.Combine(GetModuleDirectory("ChaosClothAssetDataflowNodes"), "Internal"));
		}
	}
}
