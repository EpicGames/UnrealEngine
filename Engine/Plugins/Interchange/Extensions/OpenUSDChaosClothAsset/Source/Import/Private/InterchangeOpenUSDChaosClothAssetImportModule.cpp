// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeOpenUSDChaosClothAssetImportModule.h"

#include "InterchangeOpenUSDChaosClothAssetPipeline.h"
#include "InterchangeOpenUSDChaosClothAssetRootSchemaHandler.h"

#include "InterchangeOpenUSDImportModule.h"
#include "InterchangeUSDPipeline.h"
#include "InterchangeUsdTranslator.h"
#include "SchemaHandlers/SchemaHandlerRegistry.h"

#include "InterchangeChaosClothAssetImportModule.h"
#include "InterchangeChaosClothAssetPipeline.h"

#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangePipelinesModule.h"

#include "InterchangeManager.h"
#include "InterchangeProjectSettings.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

void FInterchangeOpenUSDChaosClothAssetImportModule::StartupModule()
{
	using namespace UE::Interchange;
	using namespace UE::Interchange::USD;

#if USE_USD_SDK
	// Make sure some required modules are loaded before us as we'll be trying to load their assets
	FModuleManager::LoadModuleChecked<FInterchangeOpenUSDImportModule>("InterchangeOpenUSDImport");
	FModuleManager::LoadModuleChecked<FInterchangeChaosClothAssetImportModule>("InterchangeChaosClothAssetImport");
	FModuleManager::LoadModuleChecked<IInterchangePipelinesModule>("InterchangePipelines");

	// By being the plugin that "connects USD and ChaosCloth" we are in charge of registering even the main
	// ChaosCloth pipeline into the USD format's pipeline stacks

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	InterchangeManager.RegisterTranslator(UInterchangeUSDTranslator::StaticClass());

	RegisteredHandlerName = FSchemaHandlerRegistry::Register<FInterchangeOpenUSDChaosClothAssetRootSchemaHandler>();

	const static TSoftClassPtr<UInterchangeTranslatorBase> UsdTranslatorPath = TSoftClassPtr<UInterchangeTranslatorBase>(
		UInterchangeUSDTranslator::StaticClass()
	);
	const static FSoftObjectPath DefaultGraphPath{TEXT("/InterchangeOpenUSDChaosClothAsset/OpenUSDDefaultDataflowGraph.OpenUSDDefaultDataflowGraph")};
	const static FSoftObjectPath DefaultChaosClothPipelinePath{TEXT("/InterchangeOpenUSDChaosClothAsset/ChaosClothAssetPipeline.ChaosClothAssetPipeline")};
	const static FSoftObjectPath DefaultOpenUSDChaosClothAssetPipelinePath{TEXT("/InterchangeOpenUSDChaosClothAsset/OpenUSDChaosClothAssetPipeline.OpenUSDChaosClothAssetPipeline")};

	// Inject our payloads into the USD pipeline stacks if we find no other previously existing instances of those pipelines.
	// When injecting, we'll try putting our pipelines right after the generic assets pipelines.
	if (UInterchangeProjectSettings* ProjectSettings = GetMutableDefault<UInterchangeProjectSettings>())
	{
		auto EnsurePipelineRegistered = [](FInterchangePipelineStack& Stack)
		{
			for (FInterchangeTranslatorPipelines& PerTranslatorPipeline : Stack.PerTranslatorPipelines)
			{
				if (PerTranslatorPipeline.Translator != UsdTranslatorPath)
				{
					continue;
				}

				int32 ChaosClothPipelineIndex = INDEX_NONE;
				int32 OpenUSDChaosClothAssetPipelineIndex = INDEX_NONE;
				int32 BasePipelineIndex = INDEX_NONE;
				for (int32 Index = 0; Index < PerTranslatorPipeline.Pipelines.Num(); ++Index)
				{
					const FSoftObjectPath& PipelinePath = PerTranslatorPipeline.Pipelines[Index];

					UInterchangePipelineBase* Pipeline = Cast<UInterchangePipelineBase>(PipelinePath.TryLoad());
					if (Cast<UInterchangeChaosClothAssetPipeline>(Pipeline))
					{
						ChaosClothPipelineIndex = Index;
					}
					else if (Cast<UInterchangeOpenUSDChaosClothAssetPipeline>(Pipeline))
					{
						OpenUSDChaosClothAssetPipelineIndex = Index;
					}
					else if (Cast<UInterchangeGenericAssetsPipeline>(Pipeline) || Cast<UInterchangeUsdPipeline>(Pipeline))
					{
						BasePipelineIndex = Index;
					}
				}

				// Add our pipelines if they are not part of the stack already. If they are in the stack, let's presume the
				// user is manually doing something and not interfere
				//
				// Ideally we would have returned here, but currently it's still possible to have multiple
				// PerTranslatorPipeline entries for the same translator within the same stack, so let's try patching them all
				if (ChaosClothPipelineIndex == INDEX_NONE)
				{
					PerTranslatorPipeline.Pipelines.Insert(DefaultChaosClothPipelinePath, BasePipelineIndex + 1);
					ChaosClothPipelineIndex = BasePipelineIndex + 1;
				}

				if (OpenUSDChaosClothAssetPipelineIndex == INDEX_NONE)
				{
					PerTranslatorPipeline.Pipelines.Insert(DefaultOpenUSDChaosClothAssetPipelinePath, ChaosClothPipelineIndex + 1);
				}
			}
		};

		const FName AssetsKey = TEXT("Assets");
		if (ProjectSettings->ContentImportSettings.PipelineStacks.Contains(AssetsKey))
		{
			EnsurePipelineRegistered(ProjectSettings->ContentImportSettings.PipelineStacks[AssetsKey]);
		}

		const FName SceneKey = TEXT("Scene");
		if (ProjectSettings->SceneImportSettings.PipelineStacks.Contains(SceneKey))
		{
			EnsurePipelineRegistered(ProjectSettings->SceneImportSettings.PipelineStacks[SceneKey]);
		}
	}
#endif // USE_USD_SDK
}

void FInterchangeOpenUSDChaosClothAssetImportModule::ShutdownModule()
{
	using namespace UE::Interchange;
	using namespace UE::Interchange::USD;

#if USE_USD_SDK
	FSchemaHandlerRegistry::Unregister(RegisteredHandlerName);
#endif // USE_USD_SDK
}

IMPLEMENT_MODULE(FInterchangeOpenUSDChaosClothAssetImportModule, InterchangeOpenUSDChaosClothAssetImport)
