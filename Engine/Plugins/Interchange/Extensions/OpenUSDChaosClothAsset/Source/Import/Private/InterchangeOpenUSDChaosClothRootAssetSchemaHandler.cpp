// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "InterchangeOpenUSDChaosClothAssetRootSchemaHandler.h"

// InterchangeChaosClothAsset
#include "InterchangeChaosClothAssetDefinitions.h"
#include "InterchangeChaosClothAssetPayloadData.h"

// InterchangeOpenUSDChaosClothAsset
#include "InterchangeOpenUSDChaosClothAssetDefinitions.h"

// USD
#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDTypesConversion.h"
#include "USDValueConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdGeomSubset.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/VtValue.h"

// Chaos and ChaosCloth
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"

// Interchange
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

// Engine
#include "StaticMeshAttributes.h"

#define LOCTEXT_NAMESPACE "InterchangeOpenUSDChaosClothAssetRootSchemaHandler"

namespace UE::ClothSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	/** Most of these functions are copied from USDImportNode_v3.cpp */

	class FUsdPrimAttributeAccessor
	{
	public:
		explicit FUsdPrimAttributeAccessor(const FUsdPrim& InUsdPrim, const EUsdUpAxis UsdUpAxis = EUsdUpAxis::ZAxis)
			: UsdPrim(InUsdPrim)
			, AxesOrder(0, (UsdUpAxis == EUsdUpAxis::ZAxis) ? 1 : 2, (UsdUpAxis == EUsdUpAxis::ZAxis) ? 2 : 1)
		{
		}

		template<typename ValueType>
		ValueType GetValue(const TCHAR* AttributeName, const ValueType DefaultValue = ValueType(0)) const
		{
			if (UsdPrim.HasAttribute(AttributeName))
			{
				const FUsdAttribute UsdAttribute = UsdPrim.GetAttribute(AttributeName);
				if (UsdAttribute.HasValue())
				{
					FVtValue VtValue;
					UsdAttribute.Get(VtValue);
					return GetValueAs<ValueType>(VtValue, DefaultValue);
				}
			}
			return DefaultValue;
		}

		template<typename ValueType>
		TArray<ValueType> GetArray(const TCHAR* AttributeName) const
		{
			if (UsdPrim.HasAttribute(AttributeName))
			{
				const FUsdAttribute UsdAttribute = UsdPrim.GetAttribute(AttributeName);
				if (UsdAttribute.HasValue())
				{
					FVtValue VtValue;
					UsdAttribute.Get(VtValue);
					return GetArrayAs<ValueType>(VtValue);
				}
			}
			return TArray<ValueType>();
		}

		template<typename ValueType>
		ValueType GetArrayValue(const TCHAR* AttributeName, const ValueType DefaultValue = ValueType(0), const int32 ValueIndex = 0) const
		{
			const TArray<ValueType> Array = GetArray<ValueType>(AttributeName);
			return Array.IsValidIndex(ValueIndex) ? Array[ValueIndex] : DefaultValue;
		}

	private:
		template<typename ValueType>
		ValueType GetValueAs(const FVtValue& VtValue, const ValueType DefaultValue = ValueType(0)) const
		{
			unimplemented();
			return DefaultValue;
		}

		template<typename ValueType>
		TArray<ValueType> GetArrayAs(const FVtValue& VtValue) const
		{
			unimplemented();
			return TArray<ValueType>();
		}

		const FUsdPrim UsdPrim;
		const FIntVector AxesOrder;
	};

	template<>
	inline uint32 FUsdPrimAttributeAccessor::GetValueAs<uint32>(const FVtValue& VtValue, const uint32 DefaultValue) const
	{
		return VtValue.GetTypeName() == TEXT("unsigned int") ? UsdUtils::GetUnderlyingValue<uint32>(VtValue).Get(DefaultValue)
															 : DefaultValue;	// Note: FUsdAttribute.GetTypeName() would return "uint" instead!
	}

	template<>
	inline float FUsdPrimAttributeAccessor::GetValueAs<float>(const FVtValue& VtValue, const float DefaultValue) const
	{
		return VtValue.GetTypeName() == TEXT("float") ? UsdUtils::GetUnderlyingValue<float>(VtValue).Get(DefaultValue) : DefaultValue;
	}

	template<>
	inline FVector3f FUsdPrimAttributeAccessor::GetValueAs<FVector3f>(const FVtValue& VtValue, const FVector3f DefaultValue) const
	{
		if (VtValue.GetTypeName() == TEXT("GfVec3f"))	 // Note: FUsdAttribute.GetTypeName() would return "float3" instead!
		{
			UsdUtils::FConvertedVtValue ConvertedVtValue;
			if (UsdToUnreal::ConvertValue(VtValue, ConvertedVtValue) && !ConvertedVtValue.bIsArrayValued && !ConvertedVtValue.bIsEmpty)
			{
				return FVector3f(
					ConvertedVtValue.Entries[0][AxesOrder[0]].Get<float>(),
					ConvertedVtValue.Entries[0][AxesOrder[1]].Get<float>(),
					ConvertedVtValue.Entries[0][AxesOrder[2]].Get<float>()
				);
			}
		}
		return DefaultValue;
	}

	template<>
	inline TArray<float> FUsdPrimAttributeAccessor::GetArrayAs<float>(const FVtValue& VtValue) const
	{
		TArray<float> Array;
		if (VtValue.GetTypeName() == TEXT("VtArray<float>"))	// Note: FUsdAttribute.GetTypeName() would return "float[]" instead!
		{
			UsdUtils::FConvertedVtValue ConvertedVtValue;
			if (UsdToUnreal::ConvertValue(VtValue, ConvertedVtValue) && ConvertedVtValue.bIsArrayValued && !ConvertedVtValue.bIsEmpty)
			{
				Array.Reserve(ConvertedVtValue.Entries.Num());
				for (int32 Index = 0; Index < ConvertedVtValue.Entries.Num(); ++Index)
				{
					Array.Emplace(ConvertedVtValue.Entries[Index][0].Get<float>());
				}
			}
		}
		return Array;
	}

	FString GetStringValue(const FUsdAttribute& UsdAttribute)
	{
		if (UsdAttribute.HasValue())
		{
			FVtValue Value;
			UsdAttribute.Get(Value);
			return UsdUtils::Stringify(Value);
		}
		return FString();
	}

	TArray<int32> GetIntArrayValues(const FUsdAttribute& UsdAttribute)
	{
		using namespace UsdUtils;
		TArray<int32> IntArray;
		if (UsdAttribute.HasValue())
		{
			FVtValue Value;
			UsdAttribute.Get(Value);
			FConvertedVtValue ConvertedVtValue;
			if (UsdToUnreal::ConvertValue(Value, ConvertedVtValue) && ConvertedVtValue.bIsArrayValued && !ConvertedVtValue.bIsEmpty)
			{
				IntArray.Reserve(ConvertedVtValue.Entries.Num());
				for (const FConvertedVtValueEntry& ValueEntry : ConvertedVtValue.Entries)
				{
					IntArray.Emplace(ValueEntry[0].Get<int32>());
				}
			}
		}
		return MoveTemp(IntArray);
	}

	bool ConvertClothSolverAPISchema(const UE::FUsdPrim& Prim, UInterchangeBaseNode* TargetNode)
	{
		// Get all the instances of the ClothSolverAPI schema we have on the provided prim
		bool bFoundSchema = false;
		for (const FName& SchemaName : Prim.GetAppliedSchemas())
		{
			const static FString ClothRootAPIAndSeparator = ChaosCloth::ClothRootAPI + TEXT(":");

			// I think we could wrap some SchemaRegistry functions here but it would probably be overkill?
			FString SchemaNameStr = SchemaName.ToString();
			bool bHadPrefix = SchemaNameStr.RemoveFromStart(ClothRootAPIAndSeparator);
			if (!bHadPrefix)
			{
				continue;
			}
			bFoundSchema = true;

			// This should now hopefully be either "clo" or "chaos"
			const FString& InstanceName = SchemaNameStr;

			// e.g. "solver:clo:properties"
			const FString FullPropertyName = FString::Printf(
				TEXT("%s:%s:%s"),
				*ChaosCloth::ClothSolverAPIPropertiesPrefix,
				*InstanceName,
				*ChaosCloth::ClothSolverAPIPropertiesSuffix
			);

			UE::FUsdRelationship Relationship = Prim.GetRelationship(*FullPropertyName);
			if (!Relationship)
			{
				continue;
			}

			TArray<UE::FSdfPath> Targets;
			bool bGotTargets = Relationship.GetTargets(Targets);
			if (!bGotTargets)
			{
				continue;
			}

			for (const UE::FSdfPath& Target : Targets)
			{
				UE::FUsdPrim PropertiesPrim = Prim.GetStage().GetPrimAtPath(Target);
				if (!PropertiesPrim)
				{
					continue;
				}

				// Over here we won't really care if the prim has a solver:clo:properties that points
				// at a prim that has the ChaosSolverPropertiesAPI (we expect a CloSolverPropertiesAPI in that case),
				// although maybe that should be checked?
				TranslateAPISchemaAttributes(PropertiesPrim, ChaosCloth::ChaosSolverPropertiesAPI, TargetNode);
				TranslateAPISchemaAttributes(PropertiesPrim, ChaosCloth::CloSolverPropertiesAPI, TargetNode);
			}
		}

		return bFoundSchema;
	}

	bool IsSimMesh(const UE::FUsdPrim& Prim)
	{
		return Prim && Prim.HasAPI(*ChaosCloth::SimMeshDataAPI);
	}

	bool CheckSimMeshPrimTriangles(const FUsdPrim& SimMeshPrim, FText& OutErrorText)
	{
		const FUsdAttribute FaceVertexCountsAttr = SimMeshPrim.GetAttribute(TEXT("faceVertexCounts"));
		if (!FaceVertexCountsAttr)
		{
			OutErrorText = LOCTEXT("MissingSimMeshFaceCountAttribute", "Missing simulation mesh faceVertexCounts attribute.");
		}
		else if (FaceVertexCountsAttr.GetTypeName() != TEXT("int[]"))
		{
			OutErrorText = LOCTEXT("WrongSimMeshFaceCountTypeName", "Wrong simulation mesh faceVertexCounts type name. Needs to be 'int[]'.");
		}
		else
		{
			bool bIsTriangleMesh = true;
			const TArray<int32> FaceVertexCounts = GetIntArrayValues(FaceVertexCountsAttr);
			for (int32 FaceVertexCount : FaceVertexCounts)
			{
				if (FaceVertexCount != 3)
				{
					OutErrorText = LOCTEXT(
						"WrongSimMeshFaceCount",
						"Wrong simulation mesh face vertex count. The simulation mesh only supports '3' for triangles."
					);
					bIsTriangleMesh = false;
					break;
				}
			}
			return bIsTriangleMesh;
		}
		return false;
	}

	bool GetPatternGeomSubsetsFromMeshPrim(const FUsdPrim& MeshPrim, const FString& PatternAPI, TMap<FName, FUsdGeomSubset>& GeomSubsets, FText& OutErrorText)
	{
		GeomSubsets.Reset();
		for (const FUsdPrim& MeshChildPrim : MeshPrim.GetChildren())
		{
			if (MeshChildPrim.IsA(TEXT("GeomSubset")) && MeshChildPrim.HasAPI(*PatternAPI))
			{
				// Make a pattern name that won't collide between sim mesh and render mesh
				const FName PatternName{MeshPrim.GetName().ToString() + TEXT("__") + MeshChildPrim.GetName().ToString()};

				if (GeomSubsets.Contains(PatternName))
				{
					OutErrorText = FText::Format(LOCTEXT("DuplicatePatternGeomSubsetName", "Duplicate pattern name for GeomSubset '{0}'. The name needs to be unique."), FText::FromString(MeshChildPrim.GetPrimPath().GetString()));
					return false;
				}

				const FUsdGeomSubset GeomSubset{MeshChildPrim};

				// Read FamillyName
				const FUsdAttribute FamilyNameAttr = GeomSubset.GetFamilyNameAttr();
				if (GetStringValue(FamilyNameAttr) != TEXT("pattern"))
				{
					OutErrorText = FText::Format(LOCTEXT("WrongPatternGeomSubsetFamilyName", "Wrong pattern family name for GeomSubset '{0}'. Needs to be 'pattern'."), FText::FromString(MeshChildPrim.GetPrimPath().GetString()));
					return false;
				}

				// Read ElementTypeAttr
				const FUsdAttribute ElementTypeAttr = GeomSubset.GetElementTypeAttr();
				if (GetStringValue(ElementTypeAttr) != TEXT("face"))
				{
					OutErrorText = FText::Format(LOCTEXT("WrongPatternGeomSubsetType", "Wrong pattern type for GeomSubset '{0}'. Needs to be 'face'."), FText::FromString(MeshChildPrim.GetPrimPath().GetString()));
					return false;
				}

				// Read indices
				const FUsdAttribute IndicesAttr = GeomSubset.GetIndicesAttr();
				if (IndicesAttr.GetTypeName() != TEXT("int[]"))
				{
					OutErrorText = FText::Format(LOCTEXT("WrongPatternGeomSubsetIndexType", "Wrong pattern index type for GeomSubset '{0}'. Needs to be 'int[]'."), FText::FromString(MeshChildPrim.GetPrimPath().GetString()));
					return false;
				}

				GeomSubsets.Emplace(PatternName, GeomSubset);
			}
		}
		return true;
	}

	bool AddPatternsFromMeshPrim(const FUsdPrim& MeshPrim, const FString& PatternAPI, TMap<FName, TSet<int32>>& Patterns, FText& OutErrorText, int32 Offset = 0)
	{
		TMap<FName, FUsdGeomSubset> GeomSubsets;
		if (GetPatternGeomSubsetsFromMeshPrim(MeshPrim, PatternAPI, GeomSubsets, OutErrorText))
		{
			for (const TPair<FName, FUsdGeomSubset>& GeomSubset : GeomSubsets)
			{
				// Read indices 
				const FUsdAttribute IndicesAttr = GeomSubset.Value.GetIndicesAttr();
				if (IndicesAttr.GetTypeName() != TEXT("int[]"))
				{
					OutErrorText = FText::Format(LOCTEXT("WrongPatternGeomSubsetIndexType", "Wrong pattern index type for GeomSubset '{0}'. Needs to be 'int[]'."), FText::FromString(GeomSubset.Value.GetPrim().GetPrimPath().GetString()));
					return false;
				}
				TArray<int32> Indices = GetIntArrayValues(IndicesAttr);
				if (Offset)
				{
					for (int32& Index : Indices)
					{
						Index += Offset;
					}
				}
				Patterns.Emplace(GeomSubset.Key, MoveTemp(Indices));
			}
			return true;
		}
		return false;
	}

	bool ImportPatternsFromSimMeshPrim(const FUsdPrim& SimMeshPrim, TMap<FName, TSet<int32>>& Patterns, FText& OutErrorText)
	{
		using namespace UE::Interchange::USD::ChaosCloth;

		Patterns.Reset();
		return AddPatternsFromMeshPrim(SimMeshPrim, SimPatternAPI, Patterns, OutErrorText);
	}

	bool ImportSewingsFromSimMeshPrim(const FUsdPrim& SimMeshPrim, TMap<FName, TSet<FIntVector2>>& Sewings, FText& OutErrorText)
	{
		using namespace UE::Interchange::USD::ChaosCloth;

		Sewings.Reset();
		for (const FUsdPrim& SimMeshChildPrim : SimMeshPrim.GetChildren())
		{
			if (SimMeshChildPrim.IsA(TEXT("GeomSubset")) && SimMeshChildPrim.HasAPI(*SewingAPI))
			{
				const FUsdGeomSubset GeomSubset(SimMeshChildPrim);

				// Read FamilyName
				const FUsdAttribute FamilyNameAttr = GeomSubset.GetFamilyNameAttr();
				if (GetStringValue(FamilyNameAttr) != TEXT("sewing"))
				{
					OutErrorText = LOCTEXT("WrongSewingGeomSubsetFamilyName", "Wrong sewing GeomSubset family name. Needs to be 'sewing'.");
					return false;
				}

				// Read ElementTypeAttr
				const FUsdAttribute ElementTypeAttr = GeomSubset.GetElementTypeAttr();
				if (GetStringValue(ElementTypeAttr) != TEXT("edge"))
				{
					OutErrorText = LOCTEXT("WrongSewingGeomSubsetType", "Wrong sewing GeomSubset type. Needs to be edge.");
					return false;
				}

				// Read indices 
				const FUsdAttribute IndicesAttr = GeomSubset.GetIndicesAttr();
				if (IndicesAttr.GetTypeName() != TEXT("int[]"))
				{
					OutErrorText = LOCTEXT("WrongSewingGeomSubsetIndexType", "Wrong sewing GeomSubset index type. Needs to be int[].");
					return false;
				}

				if (Sewings.Contains(SimMeshChildPrim.GetName()))
				{
					OutErrorText = LOCTEXT("DuplicateSewingGeomSubsetName", "Duplicate sewing GeomSubset name. The name needs to be unique.");
					return false;
				}

				const TArray<int32> IntArrayValues = GetIntArrayValues(IndicesAttr);
				const int32 NumStitches = IntArrayValues.Num() / 2;
				if (NumStitches * 2 != IntArrayValues.Num())
				{
					OutErrorText = LOCTEXT("OddSewingGeomSubsetIndices", "Odd number of indices for the sewing edges.");
					return false;
				}

				TSet<FIntVector2>& Stitches = Sewings.Emplace(SimMeshChildPrim.GetName());
				Stitches.Reserve(NumStitches);
				for (int32 Index = 0; Index < NumStitches; ++Index)
				{
					const int32 Index0 = IntArrayValues[Index * 2];
					const int32 Index1 = IntArrayValues[Index * 2 + 1];
					Stitches.Emplace(Index0 <= Index1 ? FIntVector2(Index0, Index1) : FIntVector2(Index1, Index0));
				}
			}
		}
		return true;
	}

	bool ImportSpringsFromSimMeshPrim(
		const FUsdPrim& SimMeshPrim,
		FManagedArrayCollection& OutSimulationCollection,
		FText& OutErrorText)
	{
		using namespace ::Chaos::Softs;
		using namespace UE::Interchange::USD::ChaosCloth;

		const TSharedRef<FManagedArrayCollection> SimulationCollection = MakeShared<FManagedArrayCollection>(MoveTemp(OutSimulationCollection));
		ON_SCOPE_EXIT
		{
			OutSimulationCollection = MoveTemp(*SimulationCollection);
		};

		TOptional<FEmbeddedSpringConstraintFacade> SpringConstraintFacade;
		TOptional<FCollectionPropertyMutableFacade> PropertyFacade;

		for (const FUsdPrim& SimMeshChildPrim : SimMeshPrim.GetChildren())
		{
			if (SimMeshChildPrim.IsA(TEXT("GeomSubset")) && SimMeshChildPrim.HasAPI(*SpringAPI))
			{
				const FUsdGeomSubset GeomSubset(SimMeshChildPrim);

				// Read FamilyName
				const FUsdAttribute FamilyNameAttr = GeomSubset.GetFamilyNameAttr();
				if (GetStringValue(FamilyNameAttr) != TEXT("spring"))
				{
					OutErrorText = LOCTEXT("WrongSpringGeomSubsetFamilyName", "Wrong spring GeomSubset family name. Needs to be 'spring'.");
					return false;
				}

				// Read ElementTypeAttr
				const FUsdAttribute ElementTypeAttr = GeomSubset.GetElementTypeAttr();
				if (GetStringValue(ElementTypeAttr) != TEXT("edge"))
				{
					OutErrorText = LOCTEXT("WrongSpringGeomSubsetType", "Wrong spring GeomSubset type. Needs to be edge.");
					return false;
				}

				// Read indices 
				const FUsdAttribute IndicesAttr = GeomSubset.GetIndicesAttr();
				if (IndicesAttr.GetTypeName() != TEXT("int[]"))
				{
					OutErrorText = LOCTEXT("WrongSpringGeomSubsetIndexType", "Wrong spring GeomSubset index type. Needs to be int[].");
					return false;
				}

				const TArray<int32> IntArrayValues = GetIntArrayValues(IndicesAttr);
				const int32 NumSprings = IntArrayValues.Num() / 2;
				if (NumSprings * 2 != IntArrayValues.Num())
				{
					OutErrorText = LOCTEXT("OddSpringGeomSubsetIndices", "Odd number of indices for the spring edges.");
					return false;
				}

				if (NumSprings)
				{
					// Read the primvars
					FUsdPrimAttributeAccessor UsdPrimAttributeAccessor(SimMeshChildPrim);
					const TArray<float> RestLengths = UsdPrimAttributeAccessor.GetArray<float>(TEXT("primvars:restLength"));
					if (RestLengths.Num() != NumSprings)
					{
						OutErrorText = LOCTEXT("MistmatchedSpringGeomSubsetRestLengths", "The number of rest length values doesn't match the number of springs.");
						return false;
					}
					const TArray<float> CloSpringDamp = UsdPrimAttributeAccessor.GetArray<float>(TEXT("primvars:clo:springDamp"));
					const TArray<float> CloSpringStiffness = UsdPrimAttributeAccessor.GetArray<float>(TEXT("primvars:clo:springStiffness"));
					const bool bHasCloSpringPrimvars = CloSpringDamp.Num() && CloSpringStiffness.Num();  // Only use the first value of the array, that's how it is currently exported

					// Initialize the spring constraint facade
					if (!SpringConstraintFacade.IsSet())
					{
						FEmbeddedSpringFacade SpringFacade(*SimulationCollection, UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D);
						checkf(SpringFacade.IsValid(), TEXT("FEmbeddedSpringFacade constructor should have defined the schema."));
						for (int32 ConstraintIndex = 0; ConstraintIndex < SpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
						{
							FEmbeddedSpringConstraintFacade TempSpringFacade = SpringFacade.GetSpringConstraint(ConstraintIndex);
							if (TempSpringFacade.GetConstraintEndPointNumIndices() == FUintVector2(1, 1) && TempSpringFacade.GetConstraintName() == TEXT("VertexSpringConstraint"))
							{
								SpringConstraintFacade.Emplace(MoveTemp(TempSpringFacade));
								break;
							}
						}
						if (!SpringConstraintFacade.IsSet())
						{
							SpringConstraintFacade.Emplace(SpringFacade.AddGetSpringConstraint());
							SpringConstraintFacade->Initialize(
								TConstArrayView<FIntVector2>(),
								TConstArrayView<float>(),
								TConstArrayView<float>(),
								TConstArrayView<float>(),
								TConstArrayView<float>(),
								TEXT("VertexSpringConstraint"));
						}
					}

					TArray<FIntVector2> ConstraintVertices;
					ConstraintVertices.Reserve(NumSprings);
					for (int32 Index = 0; Index < NumSprings; ++Index)
					{
						const int32 Index0 = IntArrayValues[Index * 2];
						const int32 Index1 = IntArrayValues[Index * 2 + 1];
						ConstraintVertices.Emplace(Index0 <= Index1 ? FIntVector2(Index0, Index1) : FIntVector2(Index1, Index0));
					}

					SpringConstraintFacade->Append(
						TConstArrayView<FIntVector2>(ConstraintVertices.GetData(), NumSprings),
						TConstArrayView<float>(RestLengths.GetData(), NumSprings));

					// Initialize the property facade
					if (bHasCloSpringPrimvars)
					{
						if (!PropertyFacade.IsSet())
						{
							PropertyFacade.Emplace(SimulationCollection);
							PropertyFacade->DefineSchema();
						}

						auto SetPropertyValue = [&PropertyFacade](const FName PropertyName, const float Value)
							{
								constexpr ECollectionPropertyFlags CollectionPropertyFlags = ECollectionPropertyFlags::Interpolable | ECollectionPropertyFlags::Animatable | ECollectionPropertyFlags::Enabled;
								int32 VertexSpringExtensionStiffnessIndex = PropertyFacade->GetKeyNameIndex(PropertyName);
								if (VertexSpringExtensionStiffnessIndex == INDEX_NONE)
								{
									VertexSpringExtensionStiffnessIndex = PropertyFacade->AddProperty(PropertyName, CollectionPropertyFlags);
								}
								PropertyFacade->SetValue(VertexSpringExtensionStiffnessIndex, Value);
							};

						SetPropertyValue("VertexSpringExtensionStiffness", CloSpringStiffness[0]);
						SetPropertyValue("VertexSpringCompressionStiffness", CloSpringStiffness[0]);
						SetPropertyValue("VertexSpringDamping", CloSpringDamp[0]);
					}
					break;  // Only does one springs geomesubset since that's how it is currently exported, additional ones would also require creating a new weight map (see fabric code)
				}
			}
		}
		return true;
	}

	bool ImportFabricProperties(
		const FUsdPrim& SimMeshPrim,
		const EUsdUpAxis UsdUpAxis,
		FManagedArrayCollection& OutSimulationCollection,
		TMap<FName, int32>& OutSimPatternFabricIndices,
		FText& OutErrorText
	)
	{
		using namespace Chaos::ClothAsset;

		const TSharedRef<FManagedArrayCollection> SimulationCollection = MakeShared<FManagedArrayCollection>(MoveTemp(OutSimulationCollection));
		ON_SCOPE_EXIT
		{
			OutSimulationCollection = MoveTemp(*SimulationCollection);
		};

		FCollectionClothFacade ClothFacade(SimulationCollection);
		ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Solvers | EClothCollectionExtendedSchemas::Fabrics);

		// Import the fabric properties
		auto ImportFabricProperties = [&UsdUpAxis, &ClothFacade](const FUsdGeomSubset& GeomSubset) -> int32
		{
			// Read the attributes
			const FUsdPrimAttributeAccessor UsdPrimAttributeAccessor(GeomSubset.GetPrim(), UsdUpAxis);

			constexpr float BendingScaling = 1e-5f;			// From g.mm2/s2 to kg.cm2/s2
			constexpr float StretchShearScaling = 1e-3f;	// From g/s2 to kg/s2
			constexpr float DensityScaling = 1e+3f;			// From g/mm2 to kg/m2
			constexpr float ThicknessScaling = 1e-1f;		// From mm to cm

			float BendingBiasLeft = 0.0f;
			BendingBiasLeft = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bendingBiasLeft"), BendingBiasLeft) * BendingScaling;

			float BendingBiasRight = 0.0f;
			BendingBiasRight = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bendingBiasRight"), BendingBiasRight)
							   * BendingScaling;

			float BendingWarp = 0.0f;
			BendingWarp = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bendingWarp"), BendingWarp) * BendingScaling;

			float BendingWeft = 0.0f;
			BendingWeft = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bendingWeft"), BendingWeft) * BendingScaling;

			float BucklingRatioBiasLeft = 0.0f;
			BucklingRatioBiasLeft = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bucklingRatioBiasLeft"), BucklingRatioBiasLeft);

			float BucklingRatioBiasRight = 0.0f;
			BucklingRatioBiasRight = UsdPrimAttributeAccessor.GetArrayValue<float>(
				TEXT("primvars:clo:bucklingRatioBiasRight"),
				BucklingRatioBiasRight
			);

			float BucklingRatioWarp = 0.0f;
			BucklingRatioWarp = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bucklingRatioWarp"), BucklingRatioWarp);

			float BucklingRatioWeft = 0.0f;
			BucklingRatioWeft = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bucklingRatioWeft"), BucklingRatioWeft);

			float BucklingStiffnessBiasLeft = 0.0f;
			BucklingStiffnessBiasLeft = UsdPrimAttributeAccessor.GetArrayValue<float>(
				TEXT("primvars:clo:bucklingStiffnessBiasLeft"),
				BucklingStiffnessBiasLeft
			);

			float BucklingStiffnessBiasRight = 0.0f;
			BucklingStiffnessBiasRight = UsdPrimAttributeAccessor.GetArrayValue<float>(
				TEXT("primvars:clo:bucklingStiffnessBiasRight"),
				BucklingStiffnessBiasRight
			);

			float BucklingStiffnessWarp = 0.0f;
			BucklingStiffnessWarp = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bucklingStiffnessWarp"), BucklingStiffnessWarp);

			float BucklingStiffnessWeft = 0.0f;
			BucklingStiffnessWeft = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:bucklingStiffnessWeft"), BucklingStiffnessWeft);

			float Density = 0.0f;
			Density = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:density"), Density) * DensityScaling;

			float Friction = 0.0f;
			Friction = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:friction"), Friction);

			float Damping = 0.0f;
			Damping = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:internalDamping"), Damping);

			float ShearLeft = 0.0f;
			ShearLeft = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:shearLeft"), ShearLeft) * StretchShearScaling;

			float ShearRight = 0.0f;
			ShearRight = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:shearRight"), ShearRight) * StretchShearScaling;

			float StretchWarp = 0.0f;
			StretchWarp = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:stretchWarp"), StretchWarp) * StretchShearScaling;

			float StretchWeft = 0.0f;
			StretchWeft = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:stretchWeft"), StretchWeft) * StretchShearScaling;

			float Thickness = 0.0f;
			Thickness = UsdPrimAttributeAccessor.GetArrayValue<float>(TEXT("primvars:clo:thickness"), Thickness) * ThicknessScaling;

			// Initialize a new fabric with these parameters
			const FCollectionClothFabricFacade::FAnisotropicData BendingStiffness(
				BendingWeft,
				BendingWarp,
				0.5f * (BendingBiasLeft + BendingBiasRight)
			);

			const FCollectionClothFabricFacade::FAnisotropicData StretchStiffness(StretchWeft, StretchWarp, 0.5f * (ShearLeft + ShearRight));

			const float BucklingRatio = (BucklingRatioWeft + BucklingRatioWarp + 0.5f * (BucklingRatioBiasLeft + BucklingRatioBiasRight)) / 3.f;

			const FCollectionClothFabricFacade::FAnisotropicData BucklingStiffness = BucklingRatio < UE_SMALL_NUMBER
																						 ? BendingStiffness
																						 : FCollectionClothFabricFacade::FAnisotropicData(
																							   BendingStiffness.Weft * BucklingStiffnessWeft,
																							   BendingStiffness.Warp * BucklingStiffnessWarp,
																							   BendingStiffness.Bias * 0.5f
																								   * (BucklingStiffnessBiasLeft
																									  + BucklingStiffnessBiasRight)
																						   );

			const int32 FabricIndex = ClothFacade.AddFabric();
			FCollectionClothFabricFacade Fabric = ClothFacade.GetFabric(FabricIndex);

			Fabric.Initialize(BendingStiffness, BucklingRatio, BucklingStiffness, StretchStiffness, Density, Friction, Damping, 0.0f, 0, Thickness);

			return FabricIndex;
		};

		TMap<FName, FUsdGeomSubset> GeomSubsets;
		if (!GetPatternGeomSubsetsFromMeshPrim(SimMeshPrim, Interchange::USD::ChaosCloth::SimPatternAPI, GeomSubsets, OutErrorText))
		{
			return false;
		}

		for (const TPair<FName, FUsdGeomSubset>& GeomSubset : GeomSubsets)
		{
			// Setup a new fabric for this pattern
			const int32 FabricIndex = ImportFabricProperties(GeomSubset.Value);
			OutSimPatternFabricIndices.Emplace(GeomSubset.Key, FabricIndex);
		}
		return true;
	}

	FVector2f GetSimMeshUVScale(const FUsdPrim& SimMeshPrim)
	{
		// TODO: Better way of getting the attribute...
		FVector2f UVScale(1.f);
		const FUsdAttribute RestPositionScaleAttr = SimMeshPrim.GetAttribute(*Interchange::USD::ChaosCloth::SimMeshDataRestPositionScale);
		if (RestPositionScaleAttr.HasValue() && RestPositionScaleAttr.GetTypeName() == TEXT("float2"))
		{
			FVtValue Value;
			RestPositionScaleAttr.Get(Value);
			UsdUtils::FConvertedVtValue ConvertedVtValue;
			if (UsdToUnreal::ConvertValue(Value, ConvertedVtValue) && !ConvertedVtValue.bIsArrayValued && !ConvertedVtValue.bIsEmpty
				&& ConvertedVtValue.Entries.Num() == 1 && ConvertedVtValue.Entries[0].Num() == 2 && ConvertedVtValue.Entries[0][0].IsType<float>())
			{
				UVScale = FVector2f(ConvertedVtValue.Entries[0][0].Get<float>(), ConvertedVtValue.Entries[0][1].Get<float>());
			}
		}
		return UVScale;
	}

	bool ImportSimStaticMesh(
		const FUsdPrim& SimMeshPrim,
		const TMap<FName, TSet<int32>>& SimPatterns,
		const TMap<FName, TSet<FIntVector2>>& Sewings,
		const TMap<FName, int32>& SimPatternFabricIndices,
		const FManagedArrayCollection& SimulationCollection,
		const TSharedRef<FManagedArrayCollection> ClothCollection,
		FText& OutErrorText
	)
	{
		using namespace UE::Chaos::ClothAsset;
		using namespace ::Chaos::Softs;

		FCollectionClothFacade ClothFacade{ClothCollection};
		if (!ClothFacade.IsValid())
		{
			ClothFacade.DefineSchema();
		}

		FCollectionClothSelectionFacade ClothSelectionFacade{ClothCollection};
		if (!ClothSelectionFacade.IsValid())
		{
			ClothSelectionFacade.DefineSchema();
		}

		// Empty the current sim mesh and any previously created selection set
		FClothGeometryTools::DeleteSimMesh(ClothCollection);
		FClothGeometryTools::DeleteSelections(ClothCollection, ClothCollectionGroup::SimFaces);

		// Translate the actual mesh prim data, which we will convert into the simulation mesh
		FMeshDescription MeshDescription;
		FStaticMeshAttributes StaticMeshAttributes{MeshDescription};
		{
			UsdUtils::FUsdPrimMaterialAssignmentInfo TempMaterialInfo;

			StaticMeshAttributes.Register();

			UsdToUnreal::FUsdMeshConversionOptions Options;
			Options.PurposesToLoad = EUsdPurpose::Proxy | EUsdPurpose::Render | EUsdPurpose::Guide;
			Options.bMergeIdenticalMaterialSlots = true;

			bool bSuccess = UsdToUnreal::ConvertGeomMesh(SimMeshPrim, MeshDescription, TempMaterialInfo, Options);
			if (!bSuccess)
			{
				OutErrorText = LOCTEXT("FailedToImportMesh", "Failed to import Mesh data from simulation prim");
				return false;
			}
		}

		// Init the static mesh attributes
		if (!StaticMeshAttributes.GetVertexInstanceUVs().GetNumChannels())
		{
			OutErrorText = LOCTEXT("CantFindUVs", "Missing UV layer to initialize sim mesh data.");
			return false;
		}

		TArray<FVector2f> RestPositions2D;
		TArray<FVector3f> DrapedPositions3D;
		TArray<FIntVector3> TriangleToVertexIndex;

		// Retrieve 3D drapped positions
		DrapedPositions3D = StaticMeshAttributes.GetVertexPositions().GetRawArray();

		// Retrieve triangle indices and 2D rest positions
		RestPositions2D.SetNumZeroed(DrapedPositions3D.Num());

		const TConstArrayView<FVertexID> VertexInstanceVertexIndices = StaticMeshAttributes.GetVertexInstanceVertexIndices().GetRawArray();
		const TConstArrayView<FVertexInstanceID> TriangleVertexInstanceIndices = StaticMeshAttributes.GetTriangleVertexInstanceIndices().GetRawArray(
		);
		const TConstArrayView<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs().GetRawArray();

		check(TriangleVertexInstanceIndices.Num() % 3 == 0);
		TriangleToVertexIndex.SetNumUninitialized(TriangleVertexInstanceIndices.Num() / 3);

		auto SetRestPositions2D = [&RestPositions2D, &VertexInstanceUVs](FVertexID VertexID, FVertexInstanceID VertexInstanceID) -> bool
		{
			// TODO: Why is (0,0) an invalid value here? Is it not possible to get a legitimate vertex at (0,0)?
			if (RestPositions2D[VertexID] == FVector2f::Zero())
			{
				RestPositions2D[VertexID] = VertexInstanceUVs[VertexInstanceID];
			}
			else if (!RestPositions2D[VertexID].Equals(VertexInstanceUVs[VertexInstanceID]))
			{
				return false;
			}
			return true;
		};

		for (int32 TriangleIndex = 0; TriangleIndex < TriangleToVertexIndex.Num(); ++TriangleIndex)
		{
			const FVertexInstanceID VertexInstanceID0 = TriangleVertexInstanceIndices[TriangleIndex * 3];
			const FVertexInstanceID VertexInstanceID1 = TriangleVertexInstanceIndices[TriangleIndex * 3 + 1];
			const FVertexInstanceID VertexInstanceID2 = TriangleVertexInstanceIndices[TriangleIndex * 3 + 2];

			const FVertexID VertexID0 = VertexInstanceVertexIndices[VertexInstanceID0];
			const FVertexID VertexID1 = VertexInstanceVertexIndices[VertexInstanceID1];
			const FVertexID VertexID2 = VertexInstanceVertexIndices[VertexInstanceID2];

			TriangleToVertexIndex[TriangleIndex] = FIntVector3(VertexID0, VertexID1, VertexID2);

			if (!SetRestPositions2D(VertexID0, VertexInstanceID0) || !SetRestPositions2D(VertexID1, VertexInstanceID1)
				|| !SetRestPositions2D(VertexID2, VertexInstanceID2))
			{
				OutErrorText = LOCTEXT("UsdSimMeshWelded", "The sim mesh has already been welded. This importer needs an unwelded sim mesh.");
				// TODO: unweld vertices, generate seams(?), and reindex all constraints
				return false;
			}
		}

		FVector2f ImportedUVScale = GetSimMeshUVScale(SimMeshPrim);

		// Rescale the 2D mesh with the UV scale, and flip the UV's Y coordinates
		for (FVector2f& Pos : RestPositions2D)
		{
			Pos.Y = 1.f - Pos.Y;
			Pos *= ImportedUVScale;
		}

		// Save pattern to the collection cache
		check(RestPositions2D.Num() == DrapedPositions3D.Num());	// Should have already exited with the UsdSimMeshWelded error in this case
		if (TriangleToVertexIndex.Num() && RestPositions2D.Num())
		{
			// Cleanup sim mesh
			FClothDataflowTools::FSimMeshCleanup SimMeshCleanup(TriangleToVertexIndex, RestPositions2D, DrapedPositions3D);

			bool bHasRepairedTriangles = SimMeshCleanup.RemoveDegenerateTriangles();
			bHasRepairedTriangles = SimMeshCleanup.RemoveDuplicateTriangles() || bHasRepairedTriangles;

			const TArray<int32> OriginalToNewTriangles = FClothDataflowTools::GetOriginalToNewIndices<TSet<int32>>(
				SimMeshCleanup.OriginalTriangles,
				TriangleToVertexIndex.Num()
			);

			// Add support for original indices
			ClothFacade.AddUserDefinedAttribute<TArray<int32>>(UE::Interchange::ChaosCloth::OriginalIndicesName, ClothCollectionGroup::SimFaces);
			ClothFacade.AddUserDefinedAttribute<TArray<int32>>(UE::Interchange::ChaosCloth::OriginalIndicesName, ClothCollectionGroup::SimVertices2D);

			// Add the patterns from the clean mesh
			for (const TPair<FName, TSet<int32>>& PatternNameFaces : SimPatterns)
			{
				// Filter the pattern selection set using the remaining triangles from the cleaned triangle list
				TSet<int32> PatternSet;
				PatternSet.Reserve(PatternNameFaces.Value.Num());
				for (const int32 Face : PatternNameFaces.Value)
				{
					if (OriginalToNewTriangles.IsValidIndex(Face) && OriginalToNewTriangles[Face] != INDEX_NONE)
					{
						PatternSet.Emplace(OriginalToNewTriangles[Face]);
					}
				}

				// Add the new pattern
				if (PatternSet.Num())
				{
					TArray<FIntVector3> PatternTriangleToVertexIndex;
					TArray<TArray<int32>> PatternOriginalTriangles;
					PatternTriangleToVertexIndex.Reserve(PatternSet.Num());
					PatternOriginalTriangles.Reserve(PatternSet.Num());
					{
						for (const int32 Index : PatternSet)
						{
							PatternTriangleToVertexIndex.Emplace(SimMeshCleanup.TriangleToVertexIndex[Index]);
							PatternOriginalTriangles.Emplace(SimMeshCleanup.OriginalTriangles[Index].Array());
						}
					}

					TArray<FVector2f> PatternRestPositions2D;
					TArray<FVector3f> PatternDrapedPositions3D;
					TArray<TArray<int32>> PatternOriginalVertices;
					TArray<int32> PatternVertexReindex;
					const int32 MaxNumVertices = SimMeshCleanup.RestPositions2D.Num();
					PatternRestPositions2D.Reserve(MaxNumVertices);
					PatternDrapedPositions3D.Reserve(MaxNumVertices);
					PatternOriginalVertices.Reserve(MaxNumVertices);
					PatternVertexReindex.Init(INDEX_NONE, MaxNumVertices);

					int32 NewIndex = -1;
					for (FIntVector3& Triangle : PatternTriangleToVertexIndex)
					{
						for (int32 Vertex = 0; Vertex < 3; ++Vertex)
						{
							// Add the new vertex
							int32& Index = Triangle[Vertex];
							if (PatternVertexReindex[Index] == INDEX_NONE)
							{
								PatternVertexReindex[Index] = ++NewIndex;
								PatternRestPositions2D.Emplace(SimMeshCleanup.RestPositions2D[Index]);
								PatternDrapedPositions3D.Emplace(SimMeshCleanup.DrapedPositions3D[Index]);
								PatternOriginalVertices.Emplace(SimMeshCleanup.OriginalVertices[Index].Array());
							}
							// Reindex the triangle vertex with the new index
							Index = PatternVertexReindex[Index];
						}
					}

					// Find this pattern's fabric if any
					const int32 FabricIndex = SimPatternFabricIndices.Contains(PatternNameFaces.Key) ? SimPatternFabricIndices[PatternNameFaces.Key]
																									 : INDEX_NONE;

					// Add this pattern to the cloth collection
					const int32 SimPatternIndex = ClothFacade.AddSimPattern();
					FCollectionClothSimPatternFacade SimPattern = ClothFacade.GetSimPattern(SimPatternIndex);
					SimPattern.Initialize(PatternRestPositions2D, PatternDrapedPositions3D, PatternTriangleToVertexIndex, FabricIndex);

					// Keep track of the original triangle indices
					const TArrayView<TArray<int32>> OriginalTriangles = ClothFacade.GetUserDefinedAttribute<TArray<int32>>(
						UE::Interchange::ChaosCloth::OriginalIndicesName,
						ClothCollectionGroup::SimFaces
					);	  // Don't move outside the loop, the array might get re-allocated
					const int32 SimFacesOffset = SimPattern.GetSimFacesOffset();
					for (int32 Index = 0; Index < PatternOriginalTriangles.Num(); ++Index)
					{
						OriginalTriangles[SimFacesOffset + Index] = PatternOriginalTriangles[Index];
					}

					// Keep track of the original vertex indices
					const TArrayView<TArray<int32>> OriginalVertices = ClothFacade.GetUserDefinedAttribute<TArray<int32>>(
						UE::Interchange::ChaosCloth::OriginalIndicesName,
						ClothCollectionGroup::SimVertices2D
					);	  // Don't move outside the loop, the array might get re-allocated
					const int32 SimVertices2DOffset = SimPattern.GetSimVertices2DOffset();
					for (int32 Index = 0; Index < PatternOriginalVertices.Num(); ++Index)
					{
						OriginalVertices[SimVertices2DOffset + Index] = PatternOriginalVertices[Index];
					}

					// Add the pattern triangle list as a selection set
					TSet<int32>& SelectionSet = ClothSelectionFacade.FindOrAddSelectionSet(PatternNameFaces.Key, ClothCollectionGroup::SimFaces);
					SelectionSet.Empty(PatternSet.Num());
					for (int32 Index = SimFacesOffset; Index < SimFacesOffset + PatternTriangleToVertexIndex.Num(); ++Index)
					{
						SelectionSet.Emplace(Index);
					}
				}
			}

			// Check the resulting cleaned mesh
			const int32 NumSimVertices2D = ClothFacade.GetNumSimVertices2D();
			const int32 NumSimFaces = ClothFacade.GetNumSimFaces();
			if (!NumSimVertices2D || !NumSimFaces)
			{
				return true;	// Empty mesh
			}

			const TConstArrayView<TArray<int32>> OriginalVertices = ClothFacade.GetUserDefinedAttribute<TArray<int32>>(
				UE::Interchange::ChaosCloth::OriginalIndicesName,
				ClothCollectionGroup::SimVertices2D
			);
			const TArray<int32> OriginalToNewVertexIndices = FClothDataflowTools::GetOriginalToNewIndices(OriginalVertices, RestPositions2D.Num());

			// Add the sewings
			for (const TPair<FName, TSet<FIntVector2>>& SewingNameIndices : Sewings)
			{
				TSet<FIntVector2> Indices;
				for (const FIntVector2& Stitch : SewingNameIndices.Value)
				{
					if (!OriginalToNewVertexIndices.IsValidIndex(Stitch[0]) || !OriginalToNewVertexIndices.IsValidIndex(Stitch[1]))
					{
						OutErrorText = LOCTEXT("BadSewingIndex", "An out of range sewing index has been found.");
						return false;
					}
					const int32 StitchIndex0 = OriginalToNewVertexIndices[Stitch[0]];
					const int32 StitchIndex1 = OriginalToNewVertexIndices[Stitch[1]];
					if (StitchIndex0 != INDEX_NONE && StitchIndex1 != INDEX_NONE)
					{
						Indices.Emplace(
							StitchIndex0 < StitchIndex1 ? FIntVector2(StitchIndex0, StitchIndex1) : FIntVector2(StitchIndex1, StitchIndex0)
						);
					}
				}

				FCollectionClothSeamFacade ClothSeamFacade = ClothFacade.AddGetSeam();
				ClothSeamFacade.Initialize(Indices.Array());
			}

			// Add the springs
			const TSharedRef<FManagedArrayCollection> UsdClothDataCollection = MakeShared<FManagedArrayCollection>(SimulationCollection);
			const FEmbeddedSpringFacade UsdClothDataSpringFacade(*UsdClothDataCollection, ClothCollectionGroup::SimVertices3D);
			if (UsdClothDataSpringFacade.IsValid())
			{
				// Initialize the spring facade
				FEmbeddedSpringFacade SpringFacade(*ClothCollection, ClothCollectionGroup::SimVertices3D);
				checkf(SpringFacade.IsValid(), TEXT("FEmbeddedSpringFacade constructor should have defined the schema."));
				FEmbeddedSpringConstraintFacade SpringConstraintFacade = SpringFacade.AddGetSpringConstraint();
				SpringConstraintFacade.Initialize(
					TConstArrayView<FIntVector2>(),
					TConstArrayView<float>(),
					TConstArrayView<float>(),
					TConstArrayView<float>(),
					TConstArrayView<float>(),
					TEXT("VertexSpringConstraint")
				);

				for (int32 ConstraintIndex = 0; ConstraintIndex < UsdClothDataSpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
				{
					const FEmbeddedSpringConstraintFacade UsdClothDataSpringConstraintFacade = UsdClothDataSpringFacade.GetSpringConstraintConst(
						ConstraintIndex
					);
					if (UsdClothDataSpringConstraintFacade.GetConstraintEndPointNumIndices() == FUintVector2(1, 1)
						&& UsdClothDataSpringConstraintFacade.GetConstraintName() == TEXT("VertexSpringConstraint"))
					{
						const TConstArrayView<int32> SimVertex3DLookup = ClothFacade.GetSimVertex3DLookup();

						TArray<TArray<int32>> SourceIndices(UsdClothDataSpringConstraintFacade.GetSourceIndexConst());
						TArray<TArray<int32>> TargetIndices(UsdClothDataSpringConstraintFacade.GetTargetIndexConst());
						for (TArray<int32>& SourceIndex : SourceIndices)
						{
							int32 OrigIndex = SourceIndex[0];
							SourceIndex[0] = OriginalToNewVertexIndices.IsValidIndex(OrigIndex)
												&& SimVertex3DLookup.IsValidIndex(OriginalToNewVertexIndices[OrigIndex])
												 ? SimVertex3DLookup[OriginalToNewVertexIndices[OrigIndex]]
												 : INDEX_NONE;
						}
						for (TArray<int32>& TargetIndex : TargetIndices)
						{
							int32 OrigIndex = TargetIndex[0];
							TargetIndex[0] = OriginalToNewVertexIndices.IsValidIndex(OrigIndex)
												&& SimVertex3DLookup.IsValidIndex(OriginalToNewVertexIndices[OrigIndex])
												 ? SimVertex3DLookup[OriginalToNewVertexIndices[OrigIndex]]
												 : INDEX_NONE;
						}
						SpringConstraintFacade.Append(
							SourceIndices,
							UsdClothDataSpringConstraintFacade.GetSourceWeightsConst(),
							TargetIndices,
							UsdClothDataSpringConstraintFacade.GetTargetWeightsConst(),
							UsdClothDataSpringConstraintFacade.GetSpringLengthConst()
						);
					}
				}
				// Copy the properties
				const FCollectionPropertyConstFacade UsdClothDataPropertyFacade(UsdClothDataCollection);
				if (UsdClothDataPropertyFacade.IsValid())
				{
					FCollectionPropertyMutableFacade PropertyFacade(ClothCollection);
					PropertyFacade.Copy(SimulationCollection);
				}
			}

			// Add the solver properties
			const FCollectionClothConstFacade SimulationClothFacade(UsdClothDataCollection);
			if (SimulationClothFacade.IsValid(EClothCollectionExtendedSchemas::Solvers))
			{
				ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Solvers);
				ClothFacade.SetSolverAirDamping(SimulationClothFacade.GetSolverAirDamping());
				ClothFacade.SetSolverGravity(SimulationClothFacade.GetSolverGravity());
				ClothFacade.SetSolverSubSteps(SimulationClothFacade.GetSolverSubSteps());
				ClothFacade.SetSolverTimeStep(SimulationClothFacade.GetSolverTimeStep());
			}

			// Add the fabric properties
			if (SimulationClothFacade.IsValid(EClothCollectionExtendedSchemas::Fabrics))
			{
				ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Fabrics);

				for (int32 FabricIndex = 0; FabricIndex < SimulationClothFacade.GetNumFabrics(); ++FabricIndex)
				{
					verify(ClothFacade.AddFabric() == FabricIndex);
					FCollectionClothFabricFacade ClothFabricFacade = ClothFacade.GetFabric(FabricIndex);
					const FCollectionClothFabricConstFacade SimulationClothFabricFacade = SimulationClothFacade.GetFabric(FabricIndex);

					ClothFabricFacade.Initialize(
						SimulationClothFabricFacade.GetBendingStiffness(),
						SimulationClothFabricFacade.GetBucklingRatio(),
						SimulationClothFabricFacade.GetBucklingStiffness(),
						SimulationClothFabricFacade.GetStretchStiffness(),
						SimulationClothFabricFacade.GetDensity(),
						SimulationClothFabricFacade.GetFriction(),
						SimulationClothFabricFacade.GetDamping(),
						SimulationClothFabricFacade.GetPressure(),
						SimulationClothFabricFacade.GetLayer(),
						SimulationClothFabricFacade.GetCollisionThickness()
					);
				}
			}
		}
		return true;
	}
}	 // namespace UE::ClothSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("ChaosClothAssetRootHandler");
		return HandlerName;
	}

	const FString& FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Scope");
		return SchemaName;
	}

	bool FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::CanHandlePrim)

		return Prim.IsA(TEXT("Scope")) && Prim.HasAPI(*ChaosCloth::ClothRootAPI);
	}

	TOptional<bool> FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::OnTranslate)

		using namespace UE::ClothSchemaHandler::Private;
		using namespace UE::Interchange::ChaosCloth;

		UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext));
		if (!SceneNode)
		{
			return false;
		}

		// Flag this scene node as the cloth root, the cloth pipeline looks for these
		SceneNode->AddBooleanAttribute(*UE::Interchange::ChaosCloth::ClothRootTag, true);

		// Parse sim and render meshes as regular Mesh translated nodes
		TArray<FString> SimMeshNodeUids;
		TArray<UE::FSdfPath> SimMeshPrimPaths;
		TArray<FString> RenderMeshNodeUids;
		for (const UE::FUsdPrim& ChildPrim : UsdUtils::GetAllPrimsOfType(Prim, TEXT("UsdGeomMesh")))
		{
			// Here we create the mesh nodes we need, but we won't fill them with actual mesh data: The Gprim handler
			// (and any others) will find the same nodes and do that instead. This will take care of also spawning
			// scene nodes for everything, so that if we disable all the cloth parsing classes, we still end up with some kind of
			// sensible static meshes being produced
			//
			// TODO: Maybe we should wrap over code like this and have a common "GetOrCreateAssetNodeForPrim"? Because there's an
			// assumption here about the schema handlers being able to "get the node made for this prim" and yet they all must
			// construct the NodeUid in the same way to get that effect
			const FString NewNodeUid = UsdContext.MakeAssetNodeUid(ChildPrim, MeshPrefix);
			const FString NewNodeName{Prim.GetName().ToString()};
			UInterchangeMeshNode* MeshNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeMeshNode>(
				*UsdContext.GetNodeContainer(),
				NewNodeUid,
				NewNodeName
			);
			if (!MeshNode)
			{
				return false;
			}
			UE::Interchange::USD::SetPrimPath(*MeshNode, ChildPrim.GetPrimPath().GetString());

			if (IsSimMesh(ChildPrim))
			{
				TranslateAPISchemaAttributes(ChildPrim, ChaosCloth::SimMeshDataAPI, MeshNode);
				TranslateAPISchemaAttributes(ChildPrim, ChaosCloth::CloFabricAPI, MeshNode);

				SimMeshNodeUids.Add(NewNodeUid);
				SimMeshPrimPaths.Add(ChildPrim.GetPrimPath());
			}
			else
			{
				RenderMeshNodeUids.Add(NewNodeUid);
			}
		}

		// Convert solver properties
		// USDImportNode_v3 also checked the sim mesh for these as a fallback, so we do this here too
		UE::FUsdStage Stage = Prim.GetStage();
		if (!ConvertClothSolverAPISchema(Prim, SceneNode))
		{
			for (const UE::FSdfPath& SimMeshPrimPath : SimMeshPrimPaths)
			{
				UE::FUsdPrim MeshPrim = Stage.GetPrimAtPath(SimMeshPrimPath);
				if (ConvertClothSolverAPISchema(MeshPrim, SceneNode))
				{
					break;
				}
			}
		}

		// Connect scene node to the render and sim mesh nodes for easier finding later
		const static TOptional<FString> UnusedPayloadKey = {};
		UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, SimMeshesAttributeName, SimMeshNodeUids, UnusedPayloadKey);
		UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, RenderMeshesAttributeName, RenderMeshNodeUids, UnusedPayloadKey);

		return true;
	}

	bool FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::OnGetGenericPayloadData(
		const FString& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TObjectPtr<UInterchangeGenericPayloadData>& InOutPayloadData
	)
	{
		using namespace UE::ClothSchemaHandler::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeOpenUSDChaosClothAssetRootSchemaHandler::OnGetGenericPayloadData)

		const FString& PrimPath = PayloadKey;
		UE::FUsdPrim Prim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath{*PrimPath});
		UE::FUsdStage UsdStage = Prim.GetStage();

		FText ErrorText;
		auto EmitErrorText = [&Prim, &ErrorText]()
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("ChaosClothError", "Encountered an error when trying to retrieve USD Chaos Cloth payload data from prim '{0}': {1}"),
				FText::FromString(Prim.GetPrimPath().GetString()),
				ErrorText
			));
		};

		UInterchangeChaosClothAssetPayloadData* Payload = NewObject<UInterchangeChaosClothAssetPayloadData>();
		Payload->Collection = MakeShared<FManagedArrayCollection>();

		// This function is in charge of returning both the full payload for sim meshes, and also the
		// RenderPattern payload for render meshes. Luckily it seems a mesh can't be both at the same time,
		// so here we start by figuring out which mesh type we're meant to be handling
		if (IsSimMesh(Prim))
		{
			TMap<FName, TSet<int32>> SimPatterns;
			TMap<FName, TSet<FIntVector2>> Sewings;
			TMap<FName, TSet<int32>> RenderPatterns;
			TMap<FName, TSet<FName>> RenderToSimPatterns;
			TMap<FName, int32> SimPatternFabricIndices;
			FManagedArrayCollection SimulationCollection;

			if (!CheckSimMeshPrimTriangles(Prim, ErrorText))
			{
				EmitErrorText();
				return false;
			}

			if (!ImportPatternsFromSimMeshPrim(Prim, SimPatterns, ErrorText))
			{
				EmitErrorText();
				return false;
			}

			if (!ImportSewingsFromSimMeshPrim(Prim, Sewings, ErrorText))
			{
				EmitErrorText();
				return false;
			}

			if (!ImportSpringsFromSimMeshPrim(Prim, SimulationCollection, ErrorText))
			{
				EmitErrorText();
				return false;
			}

			const FUsdStageInfo UsdStageInfo{UsdStage};
			if (!ImportFabricProperties(Prim, UsdStageInfo.UpAxis, SimulationCollection, SimPatternFabricIndices, ErrorText))
			{
				EmitErrorText();
				return false;
			}

			if (!ImportSimStaticMesh(
					Prim,
					SimPatterns,
					Sewings,
					SimPatternFabricIndices,
					SimulationCollection,
					Payload->Collection.ToSharedRef(),
					ErrorText
				))
			{
				EmitErrorText();
				return false;
			}
		}
		else	// RenderMesh
		{
			// We just have to provide the render patterns here: The factory will request the render mesh payload
			// via the regular mesh payload interface, which should be handled by the Gprim schema handler
			// Import the patterns indices
			if (!AddPatternsFromMeshPrim(Prim, Interchange::USD::ChaosCloth::RenderPatternAPI, Payload->RenderPatterns, ErrorText))
			{
				EmitErrorText();
				return false;
			}
		}

		InOutPayloadData = Payload;

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
