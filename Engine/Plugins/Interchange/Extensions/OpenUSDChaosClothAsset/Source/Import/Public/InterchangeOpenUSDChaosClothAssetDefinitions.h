// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::Interchange::USD::ChaosCloth
{
	// Schema names
	inline const FString ClothRootAPI = TEXT("ClothRootAPI");
	inline const FString ChaosSolverPropertiesAPI = TEXT("ChaosSolverPropertiesAPI");
	inline const FString CloSolverPropertiesAPI = TEXT("CloSolverPropertiesAPI");
	inline const FString SimMeshDataAPI = TEXT("SimMeshDataAPI");
	inline const FString CloFabricAPI = TEXT("CloFabricAPI");
	inline const FString SimPatternAPI = TEXT("SimPatternAPI");
	inline const FString RenderPatternAPI = TEXT("RenderPatternAPI");
	inline const FString SewingAPI = TEXT("SewingAPI");
	inline const FString SpringAPI = TEXT("SpringAPI");

	// Properties of the ChaosSolverPropertiesAPI/CloSolverPropertiesAPI schemas
	inline const FString ClothSolverAPIPropertiesPrefix = TEXT("solver");
	inline const FString ClothSolverAPIPropertiesSuffix = TEXT("properties");

	// Properties of the ChaosSolverPropertiesAPI schema
	inline const FString SolverNumIterations = TEXT("numIterations");
	inline const FString SolverNumSubseps = TEXT("numSubsteps");

	// Properties of the CloSolverPropertiesAPI schema
	inline const FString SolverAirDamping = TEXT("airDamping");
	inline const FString SolverAvatarClothDetection_EdgeEdge = TEXT("avatarClothDetection_EdgeEdge");
	inline const FString SolverAvatarClothDetection_TriangleVertex = TEXT("avatarClothDetection_TriangleVertex");
	inline const FString SolverGravity = TEXT("gravity");
	inline const FString SolverIntersectionResolution = TEXT("intersectionResolution");
	inline const FString SolverLayerBasedCollisionDetection = TEXT("layerBasedCollisionDetection");
	inline const FString SolverProximityDetection_EdgeEdge = TEXT("proximityDetection_EdgeEdge");
	inline const FString SolverProximityDetection_VertexTriangle = TEXT("proximityDetection_VertexTriangle");
	inline const FString SolverSelfCollisionDetection_AvoidanceStiffness = TEXT("selfCollisionDetection_AvoidanceStiffness");
	inline const FString SolverSelfCollisionDetection_EdgeEdge = TEXT("selfCollisionDetection_EdgeEdge");
	inline const FString SolverSelfCollisionDetection_TriangleVertex = TEXT("selfCollisionDetection_TriangleVertex");
	inline const FString SolverSubStepCount = TEXT("subStepCount");
	inline const FString SolverTimeStep = TEXT("timeStep");

	// Properties of the SimMeshDataAPI schema
	inline const FString SimMeshDataRestPositionScale = TEXT("restPositionScale");

	// Properties of the CloFabricAPI schema
    inline const FString CloFabricBendingBiasLeft = TEXT("primvars:clo:bendingBiasLeft");
    inline const FString CloFabricBendingBiasRight = TEXT("primvars:clo:bendingBiasRight");
    inline const FString CloFabricBendingWarp = TEXT("primvars:clo:bendingWarp");
    inline const FString CloFabricBendingWeft = TEXT("primvars:clo:bendingWeft");
    inline const FString CloFabricBucklingRatioBiasLeft = TEXT("primvars:clo:bucklingRatioBiasLeft");
    inline const FString CloFabricBucklingRatioBiasRight = TEXT("primvars:clo:bucklingRatioBiasRight");
    inline const FString CloFabricBucklingRatioWarp = TEXT("primvars:clo:bucklingRatioWarp");
    inline const FString CloFabricBucklingRatioWeft = TEXT("primvars:clo:bucklingRatioWeft");
    inline const FString CloFabricBucklingStiffnessBiasLeft = TEXT("primvars:clo:bucklingStiffnessBiasLeft");
    inline const FString CloFabricBucklingStiffnessBiasRight = TEXT("primvars:clo:bucklingStiffnessBiasRight");
    inline const FString CloFabricBucklingStiffnessWarp = TEXT("primvars:clo:bucklingStiffnessWarp");
    inline const FString CloFabricBucklingStiffnessWeft = TEXT("primvars:clo:bucklingStiffnessWeft");
    inline const FString CloFabricDensity = TEXT("primvars:clo:density");
    inline const FString CloFabricFriction = TEXT("primvars:clo:friction");
    inline const FString CloFabricInternalDamping = TEXT("primvars:clo:internalDamping");
    inline const FString CloFabricShearLeft = TEXT("primvars:clo:shearLeft");
    inline const FString CloFabricShearRight = TEXT("primvars:clo:shearRight");
    inline const FString CloFabricShrinkage = TEXT("primvars:clo:shrinkage");
    inline const FString CloFabricStretchWarp = TEXT("primvars:clo:stretchWarp");
    inline const FString CloFabricStretchWeft = TEXT("primvars:clo:stretchWeft");
    inline const FString CloFabricThickness = TEXT("primvars:clo:thickness");
    inline const FString CloFabricBendingStiffness = TEXT("primvars:clo:bendingStiffness");
    inline const FString CloFabricBendingStiffnessScale = TEXT("primvars:clo:bendingStiffnessScale");
    inline const FString CloFabricBendingWingInversed = TEXT("primvars:clo:bendingWingInversed");
    inline const FString CloFabricBucklingRatio = TEXT("primvars:clo:bucklingRatio");
    inline const FString CloFabricBucklingScale = TEXT("primvars:clo:bucklingScale");
    inline const FString CloFabricDamp = TEXT("primvars:clo:damp");
    inline const FString CloFabricStiffness = TEXT("primvars:clo:stiffness");

	// Properties of the ChaosFabricAPI schema
	inline const FString ChaosFabricDensities = TEXT("primvars:chaos:densities");
	inline const FString ChaosFabricEdgeStiffnesses = TEXT("primvars:chaos:edgeStiffnesses");
	inline const FString ChaosFabricBendingStiffnesses = TEXT("primvars:chaos:bendingStiffnesses");
	inline const FString ChaosFabricBucklingRatios = TEXT("primvars:chaos:bucklingRatios");
	inline const FString ChaosFabricBucklingStiffnesses = TEXT("primvars:chaos:bucklingStiffnesses");
	inline const FString ChaosFabricFlatnessRatios = TEXT("primvars:chaos:flatnessRatios");
	inline const FString ChaosFabricAreaStiffnesses = TEXT("primvars:chaos:areaStiffnesses");
	inline const FString ChaosFabricCollisionThicknesses = TEXT("primvars:chaos:collisionThicknesses");
	inline const FString ChaosFabricFrictionCoefficients = TEXT("primvars:chaos:frictionCoefficients");
	inline const FString ChaosFabricDrags = TEXT("primvars:chaos:drags");
	inline const FString ChaosFabricLifts = TEXT("primvars:chaos:lifts");
	inline const FString ChaosFabricPressures = TEXT("primvars:chaos:pressures");

}	 // namespace UE::Interchange::USD::ChaosCloth
