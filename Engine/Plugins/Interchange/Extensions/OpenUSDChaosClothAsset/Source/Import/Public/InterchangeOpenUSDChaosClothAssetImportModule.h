// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Delegates/IDelegateInstance.h"

class FInterchangeOpenUSDChaosClothAssetImportModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	FString RegisteredHandlerName;
};
