// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"

#include "UObject/ObjectMacros.h"

#include "InterchangeOpenUSDChaosClothAssetPipeline.generated.h"

#define UE_API INTERCHANGEOPENUSDCHAOSCLOTHASSETIMPORT_API

class UInterchangeBaseNodeContainer;

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeOpenUSDChaosClothAssetPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	/** The name of the pipeline that will be displayed in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath PreviewSurfaceMaterialReplacement = FSoftObjectPath{TEXT("/ChaosClothAsset/Materials/USDImportMaterial.USDImportMaterial")};

	/** Note that by default we're not using translucent materials here as they don't work well for cloth at the moment */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath PreviewSurfaceTranslucentMaterialReplacement = FSoftObjectPath{TEXT("/ChaosClothAsset/Materials/USDImportMaterial.USDImportMaterial")};

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath PreviewSurfaceTwoSidedMaterialReplacement = FSoftObjectPath{TEXT("/ChaosClothAsset/Materials/USDImportTwoSidedMaterial.USDImportTwoSidedMaterial")};

	/** Note that by default we're not using translucent materials here as they don't work well for cloth at the moment */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath PreviewSurfaceTranslucentTwoSidedMaterialReplacement = FSoftObjectPath{TEXT("/ChaosClothAsset/Materials/USDImportTwoSidedMaterial.USDImportTwoSidedMaterial")};

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath DisplayColorMaterialReplacement = FSoftObjectPath{TEXT("/ChaosClothAsset/Materials/USDImportDisplayColorMaterial.USDImportDisplayColorMaterial")};

public:
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;
};

#undef UE_API
