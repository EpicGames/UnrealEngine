// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "SchemaHandlers/SchemaHandler.h"

#define UE_API INTERCHANGEOPENUSDCHAOSCLOTHASSETIMPORT_API

namespace UE::Interchange::USD
{
	class FInterchangeOpenUSDChaosClothAssetRootSchemaHandler : public FSchemaHandler
	{
	public:
		UE_API virtual const FString& GetHandlerName() const override;

		UE_API virtual const FString& GetTargetSchemaName() const override;

		UE_API virtual bool CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual TOptional<bool> CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual TOptional<bool> CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual bool OnTranslate(
			const UE::FUsdPrim& Prim,
			FTraversalInfo& TraversalInfo,
			FHandlerAccumulatedInfo& AccumulatedInfo,
			UInterchangeUsdContext& UsdContext
		) override;

		UE_API virtual bool OnGetGenericPayloadData(
			const FString& PayloadKey,
			UInterchangeUsdContext& UsdContext,
			TObjectPtr<UInterchangeGenericPayloadData>& InOutPayloadData
		);
	};
}

#undef UE_API

#endif	  // USE_USD_SDK
